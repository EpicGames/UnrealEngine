// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace UE::MathCore::Graph
{
	/** A pair of int32s represent a directed edge. The first value represents 
		* the source vertex, and the second value represents the destination 
		* vertex.
		*/
	using FDirectedEdge = TTuple<int32, int32>;

	/** A strongly connected component contains a subgraph of strongly connected
		* vertices and their corresponding edges.
		*/
	struct FStronglyConnectedComponent
	{
		/** Vertices in the strongly connected component. */
		TArray<int32> Vertices;

		/** Edges in the strongly connected component. */
		TArray<FDirectedEdge> Edges;
	};

	/** An element in a directed tree with references to children of a vertex. */
	struct FDirectedTreeElement
	{
		TArray<int32> Children;
	};

	/** A directed tree graph representation. */
	using FDirectedTree = TMap<int32, FDirectedTreeElement>;


	/** Build a directed tree from an array of edges.
		*
		* @parma InEdges - An array of directed edges.
		* @param OutTree - A tree structure built from the edges.
		*/
	MATHCORE_API void BuildDirectedTree(TArrayView<const FDirectedEdge> InEdges, FDirectedTree& OutTree);

	/** Build a transpose directed tree from an array of edges. 
		*
		* The transpose of a directed graph is created by reversing each edge.
		*
		* @parma InEdges - An array of directed edges.
		* @param OutTree - A tree structure built from the reversed edges.
		*/
	MATHCORE_API void BuildTransposeDirectedTree(TArrayView<const FDirectedEdge> InEdges, FDirectedTree& OutTree);

	/** Traverse a tree in a depth first ordering. Does not traverse the same node twice
		*
		* @param InInitialVertex - The starting vertex for traversal.
		* @param InTree          - The tree structure to traverse. 
		* @param InVisitFunc     - A function that is called for each vertex that 
		*                          is visited. If this function returns true, 
		*                          traversal is continued in a depth first 
		*                          manner. If this function returns false, the 
		*                          children of the current vertex are not visited.
		*/
	MATHCORE_API void DepthFirstNodeTraversal(int32 InInitialVertex, const FDirectedTree& InTree, TFunctionRef<bool(int32)> InVisitFunc);

	/** Traverse a tree in a depth first ordering. Does not traverse the same edge twice
		*
		* @param InInitialVertex - The starting vertex for traversal.
		* @param InTree          - The tree structure to traverse.
		* @param InVisitFunc     - A function that is called for each edge that
		*                          is visited. If this function returns true,
		*                          the children of this edge's destination vertex will be visited
		*                          in the future. If this function returns false,
		*                          then the children will not be visited (unless reaching the node via another route).
		*/
	MATHCORE_API void DepthFirstEdgeTraversal(int32 InInitialVertex, const FDirectedTree& InTree, TFunctionRef<bool(int32, int32)> InVisitFunc);

	/** Traverse a tree in a breadth first ordering. Does not traverse the same node twice
		*
		* @param InInitialVertex - The starting vertex for traversal.
		* @param InTree          - The tree structure to traverse. 
		* @param InVisitFunc     - A function that is called for each vertex that 
		*                          is visited. If this function returns true, 
		*                          the children of this vertex will be visited 
		*                          in the future. If this function returns false, 
		*                          then the children of the current vertex will 
		*                          not be visited.
		*/
	MATHCORE_API void BreadthFirstNodeTraversal(int32 InInitialVertex, const FDirectedTree& InTree, TFunctionRef<bool(int32)> InVisitFunc);

	/** Traverse a tree in a breadth first ordering. Does not traverse the same edge twice
		*
		* @param InInitialVertex - The starting vertex for traversal.
		* @param InTree          - The tree structure to traverse.
		* @param InVisitFunc     - A function that is called for each edge that
		*                          is visited. If this function returns true,
		*                          the children of this edge's destination vertex will be visited
		*                          in the future. If this function returns false,
		*                          then the children will not be visited (unless reaching the node via another route).
		*/
	MATHCORE_API void BreadthFirstEdgeTraversal(int32 InInitialVertex, const FDirectedTree& InTree, TFunctionRef<bool(int32, int32)> InVisitFunc);

	/** Returns all leaves (vertex with no children) connected with the starting vertex
		*
		* @param InInitialVertex - The starting vertex for traversal.
		* @param InTree          - The tree structure to traverse.
		* @param OutLeafVertices - List of all leaf vertices connected to the starting vertex
		*/
	MATHCORE_API void FindLeaves(int32 InInitialVertex, const FDirectedTree& InTree, TArray<int32>& OutLeafVertices);

	/** Sort vertices topologically using a depth first sorting algorithm.
		*
		* @param InUniqueVertices - An array of vertices to sort.
		* @param InUniqueEdges - An array of edges describing dependencies.
		* @param OutVertexOrder - An array where ordered vertices are placed.
		*
		* @return True if sorting was successful. False otherwise.
		*/
	MATHCORE_API bool DepthFirstTopologicalSort(TArrayView<const int32> InUniqueVertices, TArrayView<const FDirectedEdge> InUniqueEdges, TArray<int32>& OutVertexOrder);

	/** Sort vertices topologically using a Kahn's sorting algorithm.
		*
		* @param InUniqueVertices - An array of vertices to sort.
		* @param InUniqueEdges - An array of edges describing dependencies.
		* @param OutVertexOrder - An array where ordered vertices are placed.
		*
		* @return True if sorting was successful. False otherwise.
		*/
	MATHCORE_API bool KahnTopologicalSort(TArrayView<const int32> InUniqueVertices, TArrayView<const FDirectedEdge> InUniqueEdges, TArray<int32>& OutVertexOrder);


	/** Find strongly connected components given a set of edges using Tarjan
		* algorithm.
		*
		* @param InEdges - Edges in the directed graph.
		* @param OutComponents - Strongly connected components found in the graph are be added to this array.
		* @param bExcludeSingleVertex - If true, single vertices are not be returned as strongly connected components. If false, single vertices may be returned as strongly connected components. 
		*
		* @return True if one or more strongly connected components are added to OutComponents. False otherwise. 
		*/
	MATHCORE_API bool TarjanStronglyConnectedComponents(const TSet<FDirectedEdge>& InEdges, TArray<FStronglyConnectedComponent>& OutComponents, bool bExcludeSingleVertex = true);
} // namespace UE::MathCore::Graph
