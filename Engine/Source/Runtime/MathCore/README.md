
# Math Core

**MathCore** is a module meant to regroup any math-related functionality that are not "core-enough" that they should be linked against by any module having a dependency on Core (to avoid code bloat). 
Unlike Core, which is maintained by the Foundation team, maintenance and support of MathCore is shared between all Epic developers using its content. 

## When to develop in MathCore

* If a piece of code is generic enough that it has no dependency on anything else than Core but would serve a purpose for different modules in the engine. 
* If the Foundation team is not willing to make this code part of Core.
* If a piece of code is potentially a candidate for existing in Core but not likely to be used widely enough yet that it justifies it. In time, with the Foundation team's assent, it's possible that such functions would move from MathCore to Core if their usage justifies it (again, it's a question of maintenance).

## When not to develop in MathCore

* If a piece of code is not math-related. 
* If a function or piece of code would manifestly benefit for any user of the Core module (with the Foundation team's assent, of course)

## When to/not to move code in MathCore

* When the above conditions are all met.
* If a generic piece of math code is duplicated in several modules already, it's a probably a good candidate to be moved in MathCore, as long as the original locations are updated, in order not to make this yet another duplicate of the same piece of functionality. 
* If the duplicated code differs too much and its complexity would increase too greatly if made generic, then it's probably worth keeping things separate and let each module have the ability to customize its content freely for its own specific purposes. 

## Tests

It is strongly encouraged to write Low-Level Tests when adding/modifying anything in MathCore. There's already a dedicated executable, MathCoreTests, for that purpose. It is compiling and running on all platforms and is run on CIS so it's the perfect place to extend the test coverage of MathCore code. 
